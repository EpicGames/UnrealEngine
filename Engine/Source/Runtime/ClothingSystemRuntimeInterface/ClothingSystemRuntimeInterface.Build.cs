// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ClothingSystemRuntimeInterface : ModuleRules
{
	public ClothingSystemRuntimeInterface(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "ClothSysRuntimeIntrfc";

        SetupModulePhysicsSupport(Target);

        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject"
            }
        );

		if (Target.bBuildEditor || Target.bCompileAgainstEditor)
		{
			PrivateDependencyModuleNames.Add("Slate");  // For editor notifications
		}
	}
}
