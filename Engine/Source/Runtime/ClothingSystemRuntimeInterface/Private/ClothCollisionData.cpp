// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothCollisionData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothCollisionData)

void FClothCollisionData::Reset()
{
	Spheres.Reset();
	SphereConnections.Reset();
	Convexes.Reset();
	Boxes.Reset();
	ExtrudedTaperedCapsules.Reset();
}

void FClothCollisionData::Append(const FClothCollisionData& InOther)
{
	const int32 NumSpheresBefore = Spheres.Num();
	const int32 NumSphereConnectionsBefore = SphereConnections.Num();

	Spheres.Append(InOther.Spheres);
	SphereConnections.Append(InOther.SphereConnections);

	const int32 NumSphereConnectionsAfter = SphereConnections.Num();

	if(NumSpheresBefore > 0)
	{
		// Each connection that was added needs to have its sphere indices increased to match the new spheres that were added
		for(int32 NewConnectionIndex = NumSphereConnectionsBefore; NewConnectionIndex < NumSphereConnectionsAfter; ++NewConnectionIndex)
		{
			FClothCollisionPrim_SphereConnection& Connection = SphereConnections[NewConnectionIndex];
			Connection.SphereIndices[0] += NumSpheresBefore;
			Connection.SphereIndices[1] += NumSpheresBefore;
		}
	}

	Convexes.Append(InOther.Convexes);

	Boxes.Append(InOther.Boxes);

	ExtrudedTaperedCapsules.Append(InOther.ExtrudedTaperedCapsules);
}

void FClothCollisionData::AppendTransformed(const FClothCollisionData& InOther, const TArray<FTransform>& BoneTransforms)
{
	const int32 NumSpheresBefore = Spheres.Num();
	const int32 NumConvexesBefore = Convexes.Num();
	const int32 NumBoxesBefore = Boxes.Num();
	const int32 NumExtrudedCapsulesBefore = ExtrudedTaperedCapsules.Num();

	Append(InOther);

	for (int32 Index = NumSpheresBefore; Index < Spheres.Num(); ++Index)
	{
		const FTransform& Transform = BoneTransforms[Spheres[Index].BoneIndex];
		Spheres[Index].LocalPosition = Transform.TransformPosition(Spheres[Index].LocalPosition);
	}
	for (int32 Index = NumConvexesBefore; Index < Convexes.Num(); ++Index)
	{
		const FTransform& Transform = BoneTransforms[Convexes[Index].BoneIndex];
		Convexes[Index].LocalPosition = Transform.TransformPosition(Convexes[Index].LocalPosition);
		Convexes[Index].LocalRotation = Transform.TransformRotation(Convexes[Index].LocalRotation);
	}
	for (int32 Index = NumBoxesBefore; Index < Boxes.Num(); ++Index)
	{
		const FTransform& Transform = BoneTransforms[Boxes[Index].BoneIndex];
		Boxes[Index].LocalPosition = Transform.TransformPosition(Boxes[Index].LocalPosition);
		Boxes[Index].LocalRotation = Transform.TransformRotation(Boxes[Index].LocalRotation);
	}
	for (int32 Index = NumExtrudedCapsulesBefore; Index < ExtrudedTaperedCapsules.Num(); ++Index)
	{
		const FTransform& Transform = BoneTransforms[ExtrudedTaperedCapsules[Index].BoneIndex];
		ExtrudedTaperedCapsules[Index].LocalPosition = Transform.TransformPosition(ExtrudedTaperedCapsules[Index].LocalPosition);
		ExtrudedTaperedCapsules[Index].LocalRotation = Transform.TransformRotation(ExtrudedTaperedCapsules[Index].LocalRotation);
	}
}

void FClothCollisionData::AppendUnique(const FClothCollisionData& InOther)
{
	TArray<int32> OldToNewSphereIndices;
	OldToNewSphereIndices.SetNumUninitialized(InOther.Spheres.Num());

	Spheres.Reserve(Spheres.Num() + InOther.Spheres.Num());
	for (int32 SphereIndex = 0; SphereIndex < InOther.Spheres.Num(); ++SphereIndex)
	{
		OldToNewSphereIndices[SphereIndex] = Spheres.AddUnique(InOther.Spheres[SphereIndex]);
	}

	SphereConnections.Reserve(SphereConnections.Num() + InOther.SphereConnections.Num());
	for (int32 SphereConnectionIndex = 0; SphereConnectionIndex < InOther.SphereConnections.Num(); ++SphereConnectionIndex)
	{
		FClothCollisionPrim_SphereConnection SphereConnection = InOther.SphereConnections[SphereConnectionIndex];
		if (OldToNewSphereIndices.IsValidIndex(SphereConnection.SphereIndices[0]) &&
			OldToNewSphereIndices.IsValidIndex(SphereConnection.SphereIndices[1]))
		{
			SphereConnection.SphereIndices[0] = OldToNewSphereIndices[SphereConnection.SphereIndices[0]];
			SphereConnection.SphereIndices[1] = OldToNewSphereIndices[SphereConnection.SphereIndices[1]];
		}
		SphereConnections.AddUnique(MoveTemp(SphereConnection));
	}

	Convexes.Reserve(Convexes.Num() + InOther.Convexes.Num());
	for (const FClothCollisionPrim_Convex& Convex : InOther.Convexes)
	{
		Convexes.AddUnique(Convex);
	}

	Boxes.Reserve(Boxes.Num() + InOther.Boxes.Num());
	for (const FClothCollisionPrim_Box& Box : InOther.Boxes)
	{
		Boxes.AddUnique(Box);
	}

	ExtrudedTaperedCapsules.Reserve(ExtrudedTaperedCapsules.Num() + InOther.ExtrudedTaperedCapsules.Num());
	for (const FClothCollisionPrim_ExtrudedTaperedCapsule& Capsule : InOther.ExtrudedTaperedCapsules)
	{
		ExtrudedTaperedCapsules.AddUnique(Capsule);
	}
}
