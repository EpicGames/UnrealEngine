// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothPhysicalMeshDataBase_Legacy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothPhysicalMeshDataBase_Legacy)

UClothPhysicalMeshDataBase_Legacy::UClothPhysicalMeshDataBase_Legacy()
	: NumFixedVerts(0)
	, MaxBoneWeights(0)
{}

UClothPhysicalMeshDataBase_Legacy::~UClothPhysicalMeshDataBase_Legacy()
{}

TArray<float>* UClothPhysicalMeshDataBase_Legacy::GetFloatArray(const uint32 Id) const
{
	check(IdToArray.Contains(Id));
	return IdToArray[Id];
}

TArray<uint32> UClothPhysicalMeshDataBase_Legacy::GetFloatArrayIds() const
{
	TArray<uint32> Keys; 
	IdToArray.GetKeys(Keys);
	return Keys;
}

TArray<TArray<float>*> UClothPhysicalMeshDataBase_Legacy::GetFloatArrays() const
{
	TArray<TArray<float>*> Values;
	IdToArray.GenerateValueArray(Values);
	return Values;
}

void UClothPhysicalMeshDataBase_Legacy::RegisterFloatArray(
	const uint32 Id,
	TArray<float> *Array)
{
	check(Id != INDEX_NONE);
	check(Array != nullptr);
	check(!IdToArray.Contains(Id) || IdToArray[Id] == Array);
	IdToArray.Add(Id, Array);
}

