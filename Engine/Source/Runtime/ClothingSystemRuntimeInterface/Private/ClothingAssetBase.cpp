// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingAssetBase.h"

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothingAssetBase)

DEFINE_LOG_CATEGORY(LogClothingAsset)

#if WITH_EDITOR
void UClothingAssetBase::PostInitProperties()
{
	Super::PostInitProperties();
	SetFlags(RF_Transactional);
}

void UClothingAssetBase::WarningNotification(const FText& Text)
{
	FNotificationInfo* const NotificationInfo = new FNotificationInfo(Text);
	NotificationInfo->ExpireDuration = 5.0f;
	FSlateNotificationManager::Get().QueueNotification(NotificationInfo);

	UE_LOGF(LogClothingAsset, Warning, "%ls", *Text.ToString());
}
#endif
