// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingSimulationFactory.h"
#include "AutoRTFM.h"
#include "HAL/IConsoleManager.h"
#include "Features/IModularFeatures.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothingSimulationFactory)

const FName IClothingSimulationFactoryClassProvider::FeatureName = TEXT("ClothingSimulationFactoryClassProvider");

namespace ClothingSimulationFactoryConsoleVariables
{
	// UE_DEPRECATED(5.7, "Use GetClothingSimulationFactory() instead.")
	TAutoConsoleVariable<FString> CVarDefaultClothingSimulationFactoryClass(
		TEXT("p.Cloth.DefaultClothingSimulationFactoryClass"),
		TEXT("ChaosClothingSimulationFactory"),  // Chaos is the default provider when Chaos Cloth is enabled
		TEXT("The class name of the default clothing simulation factory.\n")
		TEXT("Known providers are:\n")
		TEXT("ChaosClothingSimulationFactory\n")
		, ECVF_Cheat);
}

// UE_DEPRECATED(5.7, "Use GetClothingSimulationFactory() instead.")
UE_AUTORTFM_ALWAYS_OPEN
TSubclassOf<class UClothingSimulationFactory> UClothingSimulationFactory::GetDefaultClothingSimulationFactoryClass()
{
	TSubclassOf<UClothingSimulationFactory> DefaultClothingSimulationFactoryClass = nullptr;

	const FString DefaultClothingSimulationFactoryClassName = ClothingSimulationFactoryConsoleVariables::CVarDefaultClothingSimulationFactoryClass.GetValueOnAnyThread();
	
	IModularFeatures::Get().LockModularFeatureList();
	const TArray<IClothingSimulationFactoryClassProvider*> ClassProviders = IModularFeatures::Get().GetModularFeatureImplementations<IClothingSimulationFactoryClassProvider>(IClothingSimulationFactoryClassProvider::FeatureName);
	IModularFeatures::Get().UnlockModularFeatureList();
	for (const auto& ClassProvider : ClassProviders)
	{
		if (ClassProvider)
		{
			const TSubclassOf<UClothingSimulationFactory> ClothingSimulationFactoryClass = ClassProvider->GetClothingSimulationFactoryClass();
			if (ClothingSimulationFactoryClass.Get() != nullptr)
			{
				// Always set the default to the last non null factory class (in case the search for the cvar doesn't yield any results)
				DefaultClothingSimulationFactoryClass = ClothingSimulationFactoryClass;

				// Early exit if the cvar string matches
				if (ClothingSimulationFactoryClass->GetName() == DefaultClothingSimulationFactoryClassName)
				{
					break;
				}
			}
		}
	}

	return DefaultClothingSimulationFactoryClass;
}

UClothingSimulationFactory* UClothingSimulationFactory::GetClothingSimulationFactory(const UClothingAssetBase* InAsset)
{
	IModularFeatures::Get().LockModularFeatureList();
	const TArray<IClothingSimulationFactoryClassProvider*> ClassProviders = IModularFeatures::Get().GetModularFeatureImplementations<IClothingSimulationFactoryClassProvider>(IClothingSimulationFactoryClassProvider::FeatureName);
	IModularFeatures::Get().UnlockModularFeatureList();

	for (const IClothingSimulationFactoryClassProvider* const ClassProvider : ClassProviders)
	{
		if (ClassProvider)
		{
			const TSubclassOf<UClothingSimulationFactory> ClothingSimulationFactoryClass = ClassProvider->GetClothingSimulationFactoryClass();
			UClothingSimulationFactory* const ClothingSimulationFactory = ClothingSimulationFactoryClass.GetDefaultObject();
			if (ClothingSimulationFactory && ClothingSimulationFactory->SupportsAsset(InAsset))
			{
				return ClothingSimulationFactory;
			}
		}
	}

	return nullptr;
}

