// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingSimulationInstance.h"
#include "ClothCollisionData.h"
#include "ClothingSimulationInteractor.h"
#include "ClothingSimulationFactory.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothingSimulationInstance)

FClothingSimulationInstance::FClothingSimulationInstance(USkeletalMeshComponent* SkeletalMeshComponent, const FFactoryAssetGroup& FactoryAssetGroup)
	: ClothingSimulation(static_cast<const UClothingSimulationFactory*>(FactoryAssetGroup.ClothingSimulationFactory)->CreateSimulation())  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	, ClothingSimulationContext(ClothingSimulation ? ClothingSimulation->CreateContext() : nullptr)
	, ClothingSimulationInteractor(static_cast<const UClothingSimulationFactory*>(FactoryAssetGroup.ClothingSimulationFactory)->SupportsRuntimeInteraction() ? FactoryAssetGroup.ClothingSimulationFactory->CreateInteractor() : nullptr)
	, ClothingSimulationFactory(FactoryAssetGroup.ClothingSimulationFactory)
{
	check(ClothingSimulation);
	ClothingSimulation->Initialize();

	constexpr bool bIsInitialization = true;
	FillContextAndPrepareTick(SkeletalMeshComponent, 0.f, bIsInitialization);

	for (int32 ClothingAssetIndex = 0; ClothingAssetIndex < FactoryAssetGroup.ClothingAssets.Num(); ++ClothingAssetIndex)
	{
		if (const UClothingAssetBase* const ClothingAsset = FactoryAssetGroup.ClothingAssets[ClothingAssetIndex])
		{
			ClothingSimulation->CreateActor(SkeletalMeshComponent, ClothingAsset, ClothingAssetIndex);

			if (ClothingSimulationInteractor)
			{
				ClothingSimulationInteractor->CreateClothingInteractor(ClothingAsset, ClothingAssetIndex);
				ClothingSimulationInteractor->Sync(ClothingSimulation, ClothingSimulationContext);  // An early sync is required by some interactors to complete the initializations
			}
		}
	}
	ClothingSimulation->EndCreateActor();
}

FClothingSimulationInstance::FClothingSimulationInstance() = default;

FClothingSimulationInstance::~FClothingSimulationInstance()
{
	if (ClothingSimulation)
	{
		if (ClothingSimulationContext)
		{
			ClothingSimulation->DestroyContext(ClothingSimulationContext);
		}
		ClothingSimulation->DestroyActors();
		ClothingSimulation->Shutdown();
		if (ClothingSimulationFactory)
		{
			ClothingSimulationFactory->DestroySimulation(ClothingSimulation);
		}
	}
}

void FClothingSimulationInstance::RemoveAllClothingActors()
{
	if (ClothingSimulationInteractor)
	{
		ClothingSimulationInteractor->DestroyClothingInteractors();
	}
	ClothingSimulation->DestroyActors();
}

void FClothingSimulationInstance::SyncClothingInteractor()
{
	if (ClothingSimulationInteractor && ClothingSimulation->ShouldSimulateLOD(CurrentOwnerLODIndex))
	{
		ClothingSimulationInteractor->Sync(ClothingSimulation, ClothingSimulationContext);
	}
}

void FClothingSimulationInstance::FillContextAndPrepareTick(const USkeletalMeshComponent* OwnerComponent, float DeltaTime, bool bIsInitialization)
{
	if (!bIsInitialization)
	{
		CurrentOwnerLODIndex = OwnerComponent ? OwnerComponent->GetPredictedLODLevel() : INDEX_NONE;
	}
	const bool bForceTeleportResetOnly = !bIsInitialization && !ClothingSimulation->ShouldSimulateLOD(CurrentOwnerLODIndex);
	ClothingSimulation->FillContextAndPrepareTick(OwnerComponent, DeltaTime, ClothingSimulationContext, bIsInitialization, bForceTeleportResetOnly);
}
