// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingSimulationInteractor.h"
#include "ClothingAssetBase.h"
#include "ClothingSimulationInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothingSimulationInteractor)

void UClothingSimulationInteractor::CreateClothingInteractor(const UClothingAssetBase* ClothingAsset, int32 ClothingId)
{
	if (ClothingAsset)
	{
		if (UClothingInteractor* const ClothingInteractor = CreateClothingInteractor())
		{
			ClothingInteractor->ClothingId = ClothingId;

			ClothingInteractors.Emplace(ClothingAsset->GetName(), ClothingInteractor);
		}
	}
}

void UClothingSimulationInteractor::DestroyClothingInteractors()
{
	ClothingInteractors.Reset();
}

UClothingInteractor* UClothingSimulationInteractor::GetClothingInteractor(const FName ClothingAssetName) const
{
	if (const TObjectPtr<UClothingInteractor>* ClothingInteractor = ClothingInteractors.Find(ClothingAssetName))
	{
		return *ClothingInteractor;
	}

	// Returning the default object will still make this a valid ClothingInteractor pointer, but one with no type or interface
	return UClothingInteractor::StaticClass()->GetDefaultObject<UClothingInteractor>();
}

void UClothingSimulationInteractor::Sync(IClothingSimulationInterface* Simulation, const IClothingSimulationContext* Context)
{
	if (Simulation)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
		if (Simulation->DynamicCastToIClothingSimulation())
		{
			Sync(Simulation->DynamicCastToIClothingSimulation(), Context);
			return;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		LastNumCloths = Simulation->GetNumCloths();
		LastNumKinematicParticles = Simulation->GetNumKinematicParticles();
		LastNumDynamicParticles = Simulation->GetNumDynamicParticles();
		LastNumIterations = Simulation->GetNumIterations();
		LastNumSubsteps = Simulation->GetNumSubsteps();
		LastSimulationTime = Simulation->GetSimulationTime();

		for (const TPair<FName, TObjectPtr<UClothingInteractor>>& ClothingInteractor : UClothingSimulationInteractor::ClothingInteractors)
		{
			if (ClothingInteractor.Value)
			{
				ClothingInteractor.Value->Sync(Simulation);
			}
		}
	}
	else
	{
		LastNumCloths = 0;
		LastNumKinematicParticles = 0;
		LastNumDynamicParticles = 0;
		LastNumIterations = 0;
		LastNumSubsteps = 0;
		LastSimulationTime = 0.f;
	}
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
void UClothingSimulationInteractor::Sync(IClothingSimulation* Simulation, IClothingSimulationContext* /*Context*/)  // TODO: 5.9 remove pragmas and replace IClothingSimulation with IClothingSimulationInterface
{
	if (Simulation)
	{
		LastNumCloths = Simulation->GetNumCloths();
		LastNumKinematicParticles = Simulation->GetNumKinematicParticles();
		LastNumDynamicParticles = Simulation->GetNumDynamicParticles();
		LastNumIterations = Simulation->GetNumIterations();
		LastNumSubsteps = Simulation->GetNumSubsteps();
		LastSimulationTime = Simulation->GetSimulationTime();

		for (const auto& ClothingInteractor : UClothingSimulationInteractor::ClothingInteractors)
		{
			if (ClothingInteractor.Value)
			{
				ClothingInteractor.Value->Sync(static_cast<IClothingSimulationInterface*>(Simulation));  // TODO: 5.9 remove cast to use the correct interface
			}
		}
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

