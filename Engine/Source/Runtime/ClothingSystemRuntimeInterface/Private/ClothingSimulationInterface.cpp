// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingSimulationInterface.h"

//==============================================================================
// IClotingSimulationContext
//==============================================================================

IClothingSimulationContext::IClothingSimulationContext() = default;
IClothingSimulationContext::~IClothingSimulationContext() = default;

//==============================================================================
// IClothingSimulationInterface
//==============================================================================

IClothingSimulationInterface::IClothingSimulationInterface() = default;
IClothingSimulationInterface::~IClothingSimulationInterface() = default;

//==============================================================================
// IClothingSimulation
//==============================================================================

IClothingSimulation::IClothingSimulation()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bIsLegacyInterface = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
IClothingSimulation::~IClothingSimulation() = default;
