// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingSimulationTeleportHelpers.h"
#include "ClothingSystemRuntimeTypes.h"
#include "HAL/IConsoleManager.h"

static TAutoConsoleVariable<bool> CVarClothTeleportOverride(TEXT("p.Cloth.TeleportOverride"), false, TEXT("Force console variable teleport override values over skeletal mesh properties.\n Default: false."));
static TAutoConsoleVariable<bool> CVarClothResetAfterTeleport(TEXT("p.Cloth.ResetAfterTeleport"), true, TEXT("Require p.Cloth.TeleportOverride. Reset the clothing after moving the clothing position (called teleport).\n Default: true."));
static TAutoConsoleVariable<float> CVarClothTeleportDistanceThreshold(TEXT("p.Cloth.TeleportDistanceThreshold"), 300.f, TEXT("Require p.Cloth.TeleportOverride. Conduct teleportation if the character's movement is greater than this threshold in 1 frame.\n Zero or negative values will skip the check.\n Default: 300."));
static TAutoConsoleVariable<float> CVarClothTeleportRotationThreshold(TEXT("p.Cloth.TeleportRotationThreshold"), 0.f, TEXT("Require p.Cloth.TeleportOverride. Rotation threshold in degrees, ranging from 0 to 180.\n Conduct teleportation if the character's rotation is greater than this threshold in 1 frame.\n Zero or negative values will skip the check.\n Default 0."));

namespace UE::ClothingSimulation::TeleportHelpers
{
	EClothingTeleportMode CalculateClothingTeleport(EClothingTeleportMode CurrentTeleportMode, const FMatrix& CurRootBoneMat, const FMatrix& PrevRootBoneMat, bool bResetAfterTeleport, float ClothTeleportDistThresholdSquared, float ClothTeleportCosineThresholdInRad)
	{
		EClothingTeleportMode ClothTeleportMode = CurrentTeleportMode;

		// CVar overrides
		bool bResetAfterTeleportOverride;
		float ClothTeleportDistThresholdSquaredOverride;
		float ClothTeleportCosineThresholdInRadOverride;

		if (CVarClothTeleportOverride.GetValueOnGameThread())
		{
			bResetAfterTeleportOverride = CVarClothResetAfterTeleport.GetValueOnGameThread();
			const float TeleportDistanceThresholdOverride = CVarClothTeleportDistanceThreshold.GetValueOnGameThread();
			ClothTeleportDistThresholdSquaredOverride = TeleportDistanceThresholdOverride > 0.f ? FMath::Square(TeleportDistanceThresholdOverride) : 0.f;
			const float TeleportRotationThresholdOverride = CVarClothTeleportRotationThreshold.GetValueOnGameThread();
			ClothTeleportCosineThresholdInRadOverride = FMath::Cos(FMath::DegreesToRadians(TeleportRotationThresholdOverride));
		}
		else
		{
			bResetAfterTeleportOverride = bResetAfterTeleport;
			ClothTeleportDistThresholdSquaredOverride = ClothTeleportDistThresholdSquared;
			ClothTeleportCosineThresholdInRadOverride = ClothTeleportCosineThresholdInRad;
		}

		// distance check 
		// TeleportDistanceThreshold is greater than Zero and not teleported yet
		if (ClothTeleportDistThresholdSquaredOverride > 0 && ClothTeleportMode == EClothingTeleportMode::None)
		{
			float DistSquared = FVector::DistSquared(PrevRootBoneMat.GetOrigin(), CurRootBoneMat.GetOrigin());
			if (DistSquared > ClothTeleportDistThresholdSquaredOverride) // if it has traveled too far
			{
				ClothTeleportMode = bResetAfterTeleportOverride ? EClothingTeleportMode::TeleportAndReset : EClothingTeleportMode::Teleport;
			}
		}

		// rotation check
		// if TeleportRotationThreshold is greater than Zero and the user didn't do force teleport
		if (ClothTeleportCosineThresholdInRadOverride < 1 && ClothTeleportMode == EClothingTeleportMode::None)
		{
			// Detect whether teleportation is needed or not
			// Rotation matrix's transpose means an inverse but can't use a transpose because this matrix includes scales
			FMatrix AInvB = CurRootBoneMat * PrevRootBoneMat.InverseFast();
			float Trace = AInvB.M[0][0] + AInvB.M[1][1] + AInvB.M[2][2];
			float CosineTheta = (Trace - 1.0f) / 2.0f; // trace = 1+2cos(theta) for a 3x3 matrix

			if (CosineTheta < ClothTeleportCosineThresholdInRadOverride) // has the root bone rotated too much
			{
				ClothTeleportMode = bResetAfterTeleportOverride ? EClothingTeleportMode::TeleportAndReset : EClothingTeleportMode::Teleport;
			}
		}

		return ClothTeleportMode;
	}
}
