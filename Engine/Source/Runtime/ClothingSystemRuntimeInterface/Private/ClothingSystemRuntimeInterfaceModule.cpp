// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingSystemRuntimeInterfaceModule.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FClothingSystemRuntimeInterfaceModule, ClothingSystemRuntimeInterface);

FClothingSystemRuntimeInterfaceModule::FClothingSystemRuntimeInterfaceModule()
{
}

void FClothingSystemRuntimeInterfaceModule::StartupModule()
{
}

void FClothingSystemRuntimeInterfaceModule::ShutdownModule()
{
}
