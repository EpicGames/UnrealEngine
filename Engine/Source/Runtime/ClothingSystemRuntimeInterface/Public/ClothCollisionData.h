// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothCollisionPrim.h"
#include "Containers/Array.h"
#include "UObject/ObjectMacros.h"
#include "ClothCollisionData.generated.h"

#define UE_API CLOTHINGSYSTEMRUNTIMEINTERFACE_API

USTRUCT()
struct FClothCollisionData
{
	GENERATED_BODY()

	UE_API void Reset();

	UE_API void Append(const FClothCollisionData& InOther);

	UE_API void AppendTransformed(const FClothCollisionData& InOther, const TArray<FTransform>& BoneTransforms);

	/**
	 * Append collision data, but only the individual collision elements that don't already exist as to avoid duplicates.
	 * @Note This is an expensive operation and therefore it is best to call Append() on the first batch of collision data to merge right after a Reset().
	 */
	UE_API void AppendUnique(const FClothCollisionData& InOther);

	bool IsEmpty() const
	{
		return Spheres.IsEmpty() && SphereConnections.IsEmpty() && Convexes.IsEmpty() && Boxes.IsEmpty() && ExtrudedTaperedCapsules.IsEmpty();;
	}

	// Sphere data
	UPROPERTY(EditAnywhere, Category = Collision)
	TArray<FClothCollisionPrim_Sphere> Spheres;

	// Capsule data
	UPROPERTY(EditAnywhere, Category = Collision)
	TArray<FClothCollisionPrim_SphereConnection> SphereConnections;

	// Convex Data
	UPROPERTY(EditAnywhere, Category = Collision)
	TArray<FClothCollisionPrim_Convex> Convexes;

	// Box data
	UPROPERTY(EditAnywhere, Category = Collision)
	TArray<FClothCollisionPrim_Box> Boxes;

	// Extruded tapered capsule data
	UPROPERTY(EditAnywhere, Category = Collision)
	TArray<FClothCollisionPrim_ExtrudedTaperedCapsule> ExtrudedTaperedCapsules;
};

#undef UE_API
