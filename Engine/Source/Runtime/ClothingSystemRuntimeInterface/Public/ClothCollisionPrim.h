// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/ObjectMacros.h"
#include "ClothCollisionPrim.generated.h"

/** Data for a single sphere primitive in the clothing simulation. This can either be a 
 *  sphere on its own, or part of a capsule referenced by the indices in FClothCollisionPrim_Capsule
 */
USTRUCT()
struct FClothCollisionPrim_Sphere
{
	GENERATED_BODY()

	FClothCollisionPrim_Sphere(float InRadius = 0.0f, const FVector& InLocalPosition = FVector::ZeroVector, int32 InBoneIndex = INDEX_NONE)
		: BoneIndex(InBoneIndex)
		, Radius(InRadius)
		, LocalPosition(InLocalPosition)
	{}

	bool operator==(const FClothCollisionPrim_Sphere& Other) const
	{
		return
			BoneIndex == Other.BoneIndex &&
			FMath::IsNearlyEqual(Radius, Other.Radius) &&
			LocalPosition.Equals(Other.LocalPosition);
	}

	UPROPERTY()
	int32 BoneIndex;

	UPROPERTY()
	float Radius;

	UPROPERTY()
	FVector LocalPosition;
};

/** Data for a single connected sphere primitive. This should be configured after all spheres have
 *  been processed as they are really just indexing the existing spheres
 */
USTRUCT()
struct FClothCollisionPrim_SphereConnection
{
	GENERATED_BODY()

	FClothCollisionPrim_SphereConnection(int32 SphereIndex0 = INDEX_NONE, int32 SphereIndex1 = INDEX_NONE, const FVector& InOneSidedPlaneNormal = FVector::ZeroVector)
	{
		SphereIndices[0] = SphereIndex0;
		SphereIndices[1] = SphereIndex1;
		OneSidedPlaneNormal = InOneSidedPlaneNormal;
	}

	bool operator==(const FClothCollisionPrim_SphereConnection& Other) const
	{
		return
			SphereIndices[0] == Other.SphereIndices[0] &&
			SphereIndices[1] == Other.SphereIndices[1] &&
			OneSidedPlaneNormal == Other.OneSidedPlaneNormal;
	}

	UPROPERTY()
	int32 SphereIndices[2];

	/* Zero vector means this is not a one-sided capsule.*/
	UPROPERTY()
	FVector OneSidedPlaneNormal = FVector::ZeroVector;
};

/** Data for a convex face. */
USTRUCT()
struct FClothCollisionPrim_ConvexFace
{
	GENERATED_BODY()

	FClothCollisionPrim_ConvexFace(): Plane(ForceInit) {}

	bool operator==(const FClothCollisionPrim_ConvexFace& Other) const
	{
		return Plane.Equals(Other.Plane) && Indices == Other.Indices;
	}

	UPROPERTY()
	FPlane Plane; 

	UPROPERTY()
	TArray<int32> Indices;
};

/**
 *	Data for a single convex element
 *	A convex is a collection of planes, in which the clothing will attempt to stay outside of the
 *	shape created by the planes combined.
 */
USTRUCT()
struct FClothCollisionPrim_Convex
{
	GENERATED_BODY()

	FClothCollisionPrim_Convex()
		: BoneIndex(INDEX_NONE)
	{}

	FClothCollisionPrim_Convex(
		TArray<FClothCollisionPrim_ConvexFace>&& InFaces,
		TArray<FVector>&& InSurfacePoints,
		int32 InBoneIndex = INDEX_NONE,
		const FVector& InLocalPosition = FVector::ZeroVector,
		const FQuat& InLocalRotation = FQuat::Identity)
		: Faces(InFaces)
		, SurfacePoints(InSurfacePoints)
		, LocalPosition(InLocalPosition)
		, LocalRotation(InLocalRotation)
		, BoneIndex(InBoneIndex)
	{}

	bool operator==(const FClothCollisionPrim_Convex& Other) const
	{
		if (BoneIndex == Other.BoneIndex &&
			LocalPosition == Other.LocalPosition &&
			LocalRotation == Other.LocalRotation && 
			Faces == Other.Faces &&
			SurfacePoints.Num() == Other.SurfacePoints.Num())
		{
			for (int32 Index = 0; Index < SurfacePoints.Num(); ++Index)
			{
				if (!SurfacePoints[Index].Equals(Other.SurfacePoints[Index]))
				{
					return false;
				}
			}
			return true;
		}
		return false;
	}

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FPlane> Planes_DEPRECATED;
#endif

	UPROPERTY()
	TArray<FClothCollisionPrim_ConvexFace> Faces;

	UPROPERTY()
	TArray<FVector> SurfacePoints;  // Surface points, used by Chaos and also for visualization

	UPROPERTY()
	FVector LocalPosition = FVector::ZeroVector;

	UPROPERTY()
	FQuat LocalRotation = FQuat::Identity;

	UPROPERTY()
	int32 BoneIndex;
};

/** Data for a single box primitive. */
USTRUCT()
struct FClothCollisionPrim_Box
{
	GENERATED_BODY()

	FClothCollisionPrim_Box(
		const FVector& InLocalPosition = FVector::ZeroVector,
		const FQuat& InLocalRotation = FQuat::Identity,
		const FVector& InHalfExtents = FVector::ZeroVector,
		int32 InBoneIndex = INDEX_NONE)
		: LocalPosition(InLocalPosition)
		, LocalRotation(InLocalRotation)
		, HalfExtents(InHalfExtents)
		, BoneIndex(InBoneIndex)
	{}

	bool operator==(const FClothCollisionPrim_Box& Other) const
	{
		return
			BoneIndex == Other.BoneIndex &&
			LocalPosition.Equals(Other.LocalPosition) &&
			LocalRotation.Equals(Other.LocalRotation) &&
			HalfExtents.Equals(Other.HalfExtents);
	}

	UPROPERTY()
	FVector LocalPosition;

	UPROPERTY()
	FQuat LocalRotation;

	UPROPERTY()
	FVector HalfExtents;

	UPROPERTY()
	int32 BoneIndex;
};

/** Data for an extruded tapered capsule primitive. */
USTRUCT()
struct FClothCollisionPrim_ExtrudedTaperedCapsule
{
	GENERATED_BODY()

	explicit FClothCollisionPrim_ExtrudedTaperedCapsule(
		const FVector& InLocalPosition = FVector::ZeroVector,
		const FQuat& InLocalRotation = FQuat::Identity,
		float InLength = 0.f,
		float InWidth = 0.f,
		float InRadius1 = 0.f,
		float InRadius2 = 0.f,
		int32 InBoneIndex = INDEX_NONE)
		: LocalPosition(InLocalPosition)
		, LocalRotation(InLocalRotation)
		, Length(InLength)
		, Width(InWidth)
		, Radius1(InRadius1)
		, Radius2(InRadius2)
		, BoneIndex(InBoneIndex)
	{
	}

	bool operator==(const FClothCollisionPrim_ExtrudedTaperedCapsule& Other) const
	{
		return
			BoneIndex == Other.BoneIndex &&
			LocalPosition.Equals(Other.LocalPosition) &&
			LocalRotation.Equals(Other.LocalRotation) &&
			Length == Other.Length &&
			Width == Other.Width &&
			Radius1 == Other.Radius1 &&
			Radius2 == Other.Radius2;
	}

	UPROPERTY()
	FVector LocalPosition;

	UPROPERTY()
	FQuat LocalRotation;

	UPROPERTY()
	float Length;

	UPROPERTY()
	float Width;

	UPROPERTY()
	float Radius1;

	UPROPERTY()
	float Radius2;

	UPROPERTY()
	int32 BoneIndex;
};
