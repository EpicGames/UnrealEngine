// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ClothConfigBase.generated.h"

/**
 * Base class for simulator specific simulation controls.
 * Each cloth instance on a skeletal mesh can have a unique cloth config
 */
UCLASS(Abstract, MinimalAPI)
class UClothConfigBase : public UObject
{
	GENERATED_BODY()
public:
	CLOTHINGSYSTEMRUNTIMEINTERFACE_API UClothConfigBase();
	CLOTHINGSYSTEMRUNTIMEINTERFACE_API virtual ~UClothConfigBase();

	/** Return wherether to pre-compute self collision data. */
	virtual bool NeedsSelfCollisionData() const
	PURE_VIRTUAL(UClothConfigBase::NeedsSelfCollisionData, return false;);

	/** Return wherether to pre-compute inverse masses. */
	virtual bool NeedsInverseMasses() const
	PURE_VIRTUAL(UClothConfigBase::NeedsInverseMasses, return false;);

	/** Return wherether to pre-compute the influences. */
	virtual bool NeedsNumInfluences() const
	PURE_VIRTUAL(UClothConfigBase::NeedsNumInfluences, return false;);

	/** Return wherether to pre-compute the long range attachment tethers. */
	virtual bool NeedsTethers() const
	PURE_VIRTUAL(UClothConfigBase::NeedsTethers, return false;);

	/** Return the self collision radius to precomute self collision data. */
	virtual float GetSelfCollisionRadius() const
	PURE_VIRTUAL(UClothConfigBase::GetSelfCollisionRadius, return 0.f;);

	/** Return whether tethers need to be calculated using geodesic distances instead of eclidean. */
	virtual bool TethersUseGeodesicDistance() const
	PURE_VIRTUAL(UClothConfigBase::TethersUseGeodesicDistance, return false;);
};

/**
 * These settings are shared between all instances on a skeletal mesh
 * Deprecated, use UClothConfigBase instead.
 */
UCLASS(Abstract, Deprecated, MinimalAPI)
class UDEPRECATED_ClothSharedSimConfigBase : public UObject
{
	GENERATED_BODY()
public:
	UDEPRECATED_ClothSharedSimConfigBase() {}
	virtual ~UDEPRECATED_ClothSharedSimConfigBase() = default;

	/**
	 * Return a new updated cloth shared config migrated from this current object.
	 */
	virtual UClothConfigBase* Migrate() { return nullptr; }
};
