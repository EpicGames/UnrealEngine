// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClothVertBoneData.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Math/UnrealMathSSE.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ClothPhysicalMeshDataBase_Legacy.generated.h"

struct FClothVertBoneData;
struct FColor;

/**
 * Deprecated, use FClothPhysicalMeshData instead.
 * Simulation mesh points, topology, and spatial parameters defined on that 
 * topology.
 *
 * Created curing asset import or created from a skeletal mesh.
 */
UCLASS(MinimalAPI)
class UClothPhysicalMeshDataBase_Legacy : public UObject
{
	GENERATED_BODY()
public:
	CLOTHINGSYSTEMRUNTIMEINTERFACE_API UClothPhysicalMeshDataBase_Legacy();
	CLOTHINGSYSTEMRUNTIMEINTERFACE_API virtual ~UClothPhysicalMeshDataBase_Legacy();

	/** Retrieve a registered vertex weight array by unique @param Id. */
	CLOTHINGSYSTEMRUNTIMEINTERFACE_API TArray<float>* GetFloatArray(const uint32 Id) const;

	/** Get ids for all registered weight arrays. */
	CLOTHINGSYSTEMRUNTIMEINTERFACE_API TArray<uint32> GetFloatArrayIds() const;

	/** Get all registered weight arrays. */
	CLOTHINGSYSTEMRUNTIMEINTERFACE_API TArray<TArray<float>*> GetFloatArrays() const;

protected:
	/** Register an @param Array keyed by a unique @param Id. */
	CLOTHINGSYSTEMRUNTIMEINTERFACE_API void RegisterFloatArray(const uint32 Id, TArray<float> *Array);

public:
	// Positions of each simulation vertex
	UPROPERTY()
	TArray<FVector3f> Vertices;

	// Normal at each vertex
	UPROPERTY()
	TArray<FVector3f> Normals;

#if WITH_EDITORONLY_DATA
	// Color at each vertex
	UPROPERTY()
	TArray<FColor> VertexColors;
#endif // WITH_EDITORONLY_DATA

	// Indices of the simulation mesh triangles
	UPROPERTY()
	TArray<uint32> Indices;

	// Inverse mass for each vertex in the physical mesh
	UPROPERTY()
	TArray<float> InverseMasses;

	// Indices and weights for each vertex, used to skin the mesh to create the reference pose
	UPROPERTY()
	TArray<FClothVertBoneData> BoneData;

	// Number of fixed verts in the simulation mesh (fixed verts are just skinned and do not simulate)
	UPROPERTY()
	int32 NumFixedVerts;

	// Maximum number of bone weights of any vetex
	UPROPERTY()
	int32 MaxBoneWeights;

	// Valid indices to use for self collisions (reduced set of Indices)
	UPROPERTY()
	TArray<uint32> SelfCollisionIndices;

private:
	TMap<uint32, TArray<float>*> IdToArray;
};
