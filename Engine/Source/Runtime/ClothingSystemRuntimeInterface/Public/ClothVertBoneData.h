// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothVertBoneData.generated.h"

// Bone data for a vertex
USTRUCT()
struct FClothVertBoneData
{
	GENERATED_BODY()

	FClothVertBoneData()
		: NumInfluences(0)
	{
		FMemory::Memset(BoneIndices, (uint8)INDEX_NONE, sizeof(BoneIndices));
		FMemory::Memset(BoneWeights, 0, sizeof(BoneWeights));
	}

	// MAX_TOTAL_INFLUENCES = 12 is defined in GPUSkinPublicDefs.h, but that would 
	// require a dependency on Engine, which we can't have here.  Until that can
	// be migrated elsewhere (not to mention if), we make a redundant variable
	// with an 8 in it, that needs to stay in sync with MAX_TOTAL_INFLUENCES.
	static const int8 MaxTotalInfluences = 12;

	// Number of influences for this vertex.
	UPROPERTY()
	int32 NumInfluences;

	// Up to MAX_TOTAL_INFLUENCES bone indices that this vert is weighted to
	UPROPERTY()
	uint16 BoneIndices[MaxTotalInfluences];

	// The weights for each entry in BoneIndices
	UPROPERTY()
	float BoneWeights[MaxTotalInfluences];
};

