// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Misc/Guid.h"
#if WITH_EDITOR
#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "Templates/Tuple.h"
#endif
#include "ClothingAssetBase.generated.h"

class ITransactionObjectAnnotation;
class USkeletalMesh;
class USkinnedAsset;

#define UE_API CLOTHINGSYSTEMRUNTIMEINTERFACE_API

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogClothingAsset, Log, All);

/**
 * An interface object for any clothing asset the engine can use.
 * Any clothing asset concrete object should derive from this.
 */
UCLASS(Abstract, MinimalAPI)
class UClothingAssetBase : public UObject
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	/**
	 * Binds a clothing asset submesh to a skeletal mesh section.
	 * @param InSkelMesh Skel mesh to bind to.
	 * @param InMeshLodIndex Mesh LOD to bind this asset to.
	 * @param InSectionIndex Section in the skel mesh to replace.
	 * @param InAssetLodIndex Internal clothing LOD to use.
	 */
	virtual bool BindToSkeletalMesh(USkeletalMesh* InSkelMesh, const int32 InMeshLodIndex, const int32 InSectionIndex, const int32 InAssetLodIndex)
	PURE_VIRTUAL(UClothingAssetBase::BindToSkeletalMesh, return false;);

	/**
	 * Unbind Clothing Data from the specified Skeletal Mesh LOD sections.
	 * If MeshLodIndex is INDEX_NONE, the Clothing Data will get unbound from all LODs and all sections (ignoring SectionIndex in this case).
	 * If SectionIndex is INDEX_NONE, the Clothing Data will get unbound from all sections of the specified LOD.
	 * @param InSkelMesh The Skeletal Mesh to unbind the Clothing Data from.
	 * @param InMeshLodIndex Mesh LOD to remove this asset from (could still be bound to other LODs), or INDEX_NONE to unbind all LODs.
	 * @param InSectionIndex The Skeletal Mesh section to unbind, or INDEX_NONE to unbind all sections.
	 */
	virtual void UnbindFromSkeletalMesh(USkeletalMesh* InSkelMesh, const int32 InMeshLodIndex, const int32 InSectionIndex)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	PURE_VIRTUAL(UClothingAssetBase::UnbindFromSkeletalMesh, UnbindFromSkeletalMesh(InSkelMesh, InMeshLodIndex); return;);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.8, "Implement and use UnbindFromSkeletalMesh(USkeletalMesh* InSkelMesh, const int32 InMeshLodIndex, const int32 InSectionIndex) instead.")
	virtual void UnbindFromSkeletalMesh(USkeletalMesh* InSkelMesh) {}

	UE_DEPRECATED(5.8, "Implement and use UnbindFromSkeletalMesh(USkeletalMesh* InSkelMesh, const int32 InMeshLodIndex, const int32 InSectionIndex) instead.")
	virtual void UnbindFromSkeletalMesh(USkeletalMesh* InSkelMesh, const int32 InMeshLodIndex) {}

	/**
	 * Update all extra LOD deformer mappings.
	 * This should be called whenever the raytracing LOD bias is changed.
	 */
	virtual void UpdateAllLODBiasMappings(USkeletalMesh* SkeletalMesh)
	PURE_VIRTUAL(UClothingAssetBase::UpdateAllLODBiasMappings, return;);

	/**
	 * Return the simulation mesh.
	 * Used to calculate the LOD bias deformer mappings.
	 * @param InMeshLodIndex The bound skeletal mesh LOD (in case this clothing data hold multiple LODs).
	 * @param OutPositions The simulation mesh positions.
	 * @param OutIndices The simulation mesh triangle indices.
	 * @param OutMaxDistances The simulation mesh max distance attributes (0 for kinematic, >0 for dynamic). must match the number of positions.
	 */
	virtual void GetSimulationMesh(const int32 InMeshLodIndex, TArray<FVector3f>& OutPositions, TArray<uint32>& OutIndices, TArray<float>& OutMaxDistances) const
	PURE_VIRTUAL(UClothingAssetBase::GetSimulationMesh, return;);

	/** Add a new LOD class instance. */
	UE_DEPRECATED(5.7, "Use UClothingAssetCommon::AddNewLod() instead.")
	virtual int32 AddNewLod()
	{
		return INDEX_NONE;
	}

	/**
	 * Builds the LOD transition data
	 * When we transition between LODs we skin the incoming mesh to the outgoing mesh
	 * in exactly the same way the render mesh is skinned to create a smooth swap
	 */
	UE_DEPRECATED(5.7, "Use UClothingAssetCommon::BuildLodTransitionData() instead.")
	virtual void BuildLodTransitionData() {}

	/**
	 * Gather stats for the Skeletal Mesh editor details view.
	 * @return An array of pairs stat name, stat value that will be displayed in a row.
	 * @note Stats without values are displayed as the line header and act as a newline for the following entries.
	 */
	virtual TArray<TPair<FText, FText>> GetStats() const
	{
		return {};
	}

	/**
	 * Get a Skinned Asset dependency that participates in async compilation, if any.
	 * Used by the Skeletal Mesh to declare any compilation dependencies from the clothing assets.
	 * @return The underlying Skinned Asset or nullptr if this clothing asset has no such dependency.
	 */
	virtual USkinnedAsset* GetSkinnedAssetDependency() const { return nullptr; }

	//~ Begin UObject Interface
	/** Make sure the object is transactional. */
	UE_API virtual void PostInitProperties() override;
	//~ End UObject Interface
#endif

#if WITH_EDITORONLY_DATA
	/**
	 * Called on the clothing asset when the base data (physical mesh, config etc.)
	 * has changed, so any intermediate generated data can be regenerated.
	 */
	UE_DEPRECATED(5.7, "Use UClothingAssetCommon::InvalidateAllCachedData() instead.")
	virtual void InvalidateAllCachedData() {}
#endif // WITH_EDITORONLY_DATA

	/** 
	 * Messages to the clothing asset that the bones in the parent mesh have
	 * possibly changed, which could invalidate the bone indices stored in the LOD
	 * data.
	 * @param InSkelMesh - The mesh to use to remap the bones
	 */
	virtual void RefreshBoneMapping(USkeletalMesh* InSkelMesh)
	PURE_VIRTUAL(UClothingAssetBase::RefreshBoneMapping, return;);

	/** Check the validity of this clothing data, for example for when it is dependent from an asset that isn't ready or accessible. */
	virtual bool IsValid() const
	{
		return true;
	}

	/** Check the validity of a LOD index */
	UE_DEPRECATED(5.7, "Use UClothingAssetCommon::IsValidLod() instead.")
	virtual bool IsValidLod(int32 InLodIndex) const
	{
		return false;
	}

	/** Get the number of LODs defined in the clothing asset */
	UE_DEPRECATED(5.7, "Use UClothingAssetCommon::GetNumLods() instead.")
	virtual int32 GetNumLods() const
	{
		return 0;
	}

	/** Called after all cloth assets sharing the same simulation are added or loaded */
	virtual void PostUpdateAllAssets()
	PURE_VIRTUAL(UClothingAssetBase::PostUpdateAllAssets(), return;);

	/** Get the guid identifying this asset */
	const FGuid& GetAssetGuid() const
	{
		return AssetGuid;
	}

protected:

	/** The asset factory should have access, as it will assign the asset guid when building assets */
	friend class UClothingAssetFactory;

#if WITH_EDITOR
	/** Warn using slate's notification manager. Use this method to notify binding issues back to the user. */
	static UE_API void WarningNotification(const FText& Text);
#endif

	/** Guid to identify this asset. Will be embedded into chunks that are created using this asset */
	UPROPERTY()
	FGuid AssetGuid;
};

#undef UE_API
