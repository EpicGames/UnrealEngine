// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ClothingSimulationInterface.h"
#include "Containers/ArrayView.h"
#include "Features/IModularFeature.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ClothingSimulationFactory.generated.h"

#define UE_API CLOTHINGSYSTEMRUNTIMEINTERFACE_API

class FName;
class UClass;
class UClothConfigBase;
class UClothingAssetBase;
class UClothingSimulationInteractor;
class UEnum;

// An interface for a class that will provide default simulation factory classes
// Used by modules wanting to override clothing simulation to provide their own implementation
class IClothingSimulationFactoryClassProvider : public IModularFeature
{
public:

	// The feature name to register against for providers
	static UE_API const FName FeatureName;

	// Called by the engine to get the clothing simulation factory associated with this
	// provider for skeletal mesh components (see USkeletalMeshComponent constructor).
	// Returns Factory class for simulations or nullptr to disable clothing simulation
	virtual TSubclassOf<class UClothingSimulationFactory> GetClothingSimulationFactoryClass() const = 0;
};

// Any clothing simulation factory should derive from this interface object to interact with the engine
UCLASS(Abstract, MinimalAPI)
class UClothingSimulationFactory : public UObject
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.7, "Use GetClothingSimulationFactory() instead.")
	static UE_API TSubclassOf<class UClothingSimulationFactory> GetDefaultClothingSimulationFactoryClass();

	// Return the clothing simulation factory for the specified asset.
	// Calls SupportsAsset() internally.
	// If multiple simulation support the specified asset type then this returns the first one registered through IClothingSimulationFactoryClassProvider.
	static UE_API UClothingSimulationFactory* GetClothingSimulationFactory(const UClothingAssetBase* InAsset);

	// Create a simulation object for a skeletal mesh to use (see IClothingSimulationInterface)
	virtual IClothingSimulationInterface* CreateSimulation() const  // TODO 5.9: Made this method into a PURE_VIRTUAL
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return const_cast<UClothingSimulationFactory*>(this)->CreateSimulation();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	virtual IClothingSimulation* CreateSimulation()
	{
		return nullptr;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Destroy a simulation object, guaranteed to be a pointer returned from CreateSimulation for this factory
	virtual void DestroySimulation(IClothingSimulationInterface* InSimulation) const  // TODO 5.9: Make this method into a PURE_VIRTUAL
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (ensureMsgf(InSimulation && InSimulation->DynamicCastToIClothingSimulation(), TEXT("DestroySimulation(IClothingSimulationInterface*) must be implemented from 5.7, as the function will become pure virtual.")))
		{
			const_cast<UClothingSimulationFactory*>(this)->DestroySimulation(InSimulation->DynamicCastToIClothingSimulation());
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	virtual void DestroySimulation(IClothingSimulation* InSimulation) {}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Given an asset, decide whether this factory can create a simulation to use the data inside
	// (return false if data is invalid or missing in the case of custom data)
	virtual bool SupportsAsset(const UClothingAssetBase* InAsset) const  // TODO 5.9: Make this method into a PURE_VIRTUAL
	{
		return const_cast<UClothingSimulationFactory*>(this)->SupportsAsset(InAsset);
	}

	UE_DEPRECATED(5.7, "Use const version instead.")
	virtual bool SupportsAsset(UClothingAssetBase* InAsset)
	{
		return false;
	}

	// Whether or not we provide an interactor object to manipulate the simulation at runtime.
	// If true is returned then CreateInteractor *must* create a valid object to handle this
	virtual bool SupportsRuntimeInteraction() const  // TODO 5.9: Make this method into a PURE_VIRTUAL
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return const_cast<UClothingSimulationFactory*>(this)->SupportsRuntimeInteraction();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.7, "Use const version instead.")
	virtual bool SupportsRuntimeInteraction()
	{
		return false;
	}

	// Creates the runtime interactor object for a clothing simulation. This object will
	// receive events allowing it to write data to the simulation context in a safe manner
	virtual UClothingSimulationInteractor* CreateInteractor()
	PURE_VIRTUAL(UClothingSimulationFactory::CreateInteractor, return nullptr;);

	// Return the cloth config type for this cloth factory
	virtual TArrayView<const TSubclassOf<UClothConfigBase>> GetClothConfigClasses() const
	PURE_VIRTUAL(UClothingSimulationFactory::GetClothConfigClasses, return TArrayView<const TSubclassOf<UClothConfigBase>>(););

	// Return an enum of the weight map targets that can be used with this simulation.
	virtual const UEnum* GetWeightMapTargetEnum() const
	PURE_VIRTUAL(UClothingSimulationFactory::GetWeightMapTargetEnum, return nullptr;);
};

#undef UE_API
