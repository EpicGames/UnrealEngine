// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClothingSimulationInterface.h"
#include "Math/BoxSphereBounds.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "ClothingSimulationInstance.generated.h"

#define UE_API CLOTHINGSYSTEMRUNTIMEINTERFACE_API

struct FClothSimulData;
class UClothingSimulationFactory;
class UClothingSimulationInteractor;
class USkeletalMesh;
class USkeletalMeshComponent;
class USkinnedMeshComponent;

/**
 * Clothing simulation objects.
 * Facade to the simulation interfaces to allow for a simplified implementation and multiple simulations to coexist with each other.
 */
USTRUCT()
struct FClothingSimulationInstance
{
public:
	GENERATED_BODY()

	/**
	 * Factory asset group.
	 * Group of clothing data that run under the same simulation type, and the clothing simulation factory associated with them.
	 * The group storage array always matches the order of the SkeletalMesh clothing data so that the indices are the same,
	 * with a nullptr entry into the array to signify that the asset is part of a different group.
	 * This is to avoid an additional array of indices, knowing that most Skeletal Meshes have rarely more than half a dozen clothing data.
	 */
	struct FFactoryAssetGroup
	{
	public:
		FFactoryAssetGroup(UClothingSimulationFactory* InClothingSimulationFactory, const int32 NumTotalClothAssets)
			: ClothingSimulationFactory(InClothingSimulationFactory)
		{
			check(ClothingSimulationFactory);
			ClothingAssets.SetNumZeroed(NumTotalClothAssets);
		}

		void AddClothingAsset(const UClothingAssetBase* ClothingAsset, const int32 Index)
		{
			ClothingAssets[Index] = ClothingAsset;
		}

		const UClothingSimulationFactory* GetClothingSimulationFactory() const
		{
			return ClothingSimulationFactory;
		}

	private:
		friend FClothingSimulationInstance;
		UClothingSimulationFactory* ClothingSimulationFactory;
		TArray<const UClothingAssetBase*> ClothingAssets;
	};

	/** Construct a new clothing simulation instance for the specified asset group and component. */
	UE_API FClothingSimulationInstance(USkeletalMeshComponent* SkeletalMeshComponent, const FFactoryAssetGroup& FactoryAssetGroup);
	
	UE_API FClothingSimulationInstance();
	UE_API ~FClothingSimulationInstance();

	FClothingSimulationInstance(const FClothingSimulationInstance&) = delete;
	FClothingSimulationInstance(FClothingSimulationInstance&&) = default;
	FClothingSimulationInstance& operator=(const FClothingSimulationInstance&) = delete;
	FClothingSimulationInstance& operator=(FClothingSimulationInstance&&) = default;

	/** Return the clothing simulation factory used to initialize this clothing simulation instance. */
	UClothingSimulationFactory* GetClothingSimulationFactory() const
	{
		return ClothingSimulationFactory;
	}

	/** Return the clothing simulation pointer. */
	const IClothingSimulationInterface* GetClothingSimulation() const
	{
		return ClothingSimulation;
	}

	/** Return the clothing simulation pointer. */
	IClothingSimulationInterface* GetClothingSimulation()
	{
		return ClothingSimulation;
	}

	/** Return the clothing simulation context. */
	const IClothingSimulationContext* GetClothingSimulationContext() const
	{
		return ClothingSimulationContext;
	}

	/** Return the clothing simulation context. */
	IClothingSimulationContext* GetClothingSimulationContext()
	{
		return ClothingSimulationContext;
	}

	/** Return a non const version of the clothing interactor (non const methods are used to update simulation parameters). */
	UClothingSimulationInteractor* GetClothingSimulationInteractor() const
	{
		return ClothingSimulationInteractor;
	}

	/** Remove all actors. */
	UE_API void RemoveAllClothingActors();

	/** Synchronize the simulation with the changes requested by the interactor. */
	UE_API void SyncClothingInteractor();

	/** Hard reset the cloth simulation */
	void HardResetSimulation()
	{
		ClothingSimulation->HardResetSimulation(static_cast<const IClothingSimulationContext*>(ClothingSimulationContext));
	}

	/** Return the collision data from this simulation instance. */
	UE_DEPRECATED(5.8, "GetCollisions has been deprecated. Use CollisionSources to add external collisions instead.")
	void GetCollisions(FClothCollisionData& Collisions, bool bIncludeExternal) const {}

	/** Append unique collision data from this simulation instance to the specified collision data. */
	UE_DEPRECATED(5.8, "AppendUniqueCollisions has been deprecated. Use CollisionSources to add external collisions instead.")
	void AppendUniqueCollisions(FClothCollisionData& OutCollisions, bool bIncludeExternal) const {}

	/** Add external dynamic collisions. */
	void AddExternalCollisions(const FClothCollisionData& Collisions)
	{
		ClothingSimulation->AddExternalCollisions(Collisions);
	}

	/** Remove all dynamically added external collisions. */
	void ClearExternalCollisions()
	{
		ClothingSimulation->ClearExternalCollisions();
	}

	/** Fill the simulation context and do any other related game-thread initialization. */
	UE_API void FillContextAndPrepareTick(const USkeletalMeshComponent* OwnerComponent, float DeltaTime, bool bIsInitialization);

	/** Advance the simulation. This method may be called asynchronously. */
	void Simulate()
	{
		if (ClothingSimulation->ShouldSimulateLOD(CurrentOwnerLODIndex))
		{
			ClothingSimulation->Simulate_AnyThread(static_cast<const IClothingSimulationContext*>(ClothingSimulationContext));
		}
		else
		{
			ClothingSimulation->ForceClothNextUpdateTeleportAndReset_AnyThread();
		}
	}

	/** Append the simulation data. */
	void AppendSimulationData(TMap<int32, FClothSimulData>& CurrentSimulationData, const USkeletalMeshComponent* OwnerComponent, const USkinnedMeshComponent* LeaderPoseComponent) const
	{
		if (ClothingSimulation->ShouldSimulateLOD(CurrentOwnerLODIndex))
		{
			ClothingSimulation->AppendSimulationData(CurrentSimulationData, OwnerComponent, LeaderPoseComponent);
		}
	}


	/** Return this simulation bounds. */
	FBoxSphereBounds GetBounds(const USkeletalMeshComponent* OwnerComponent) const
	{
		if (ClothingSimulation->ShouldSimulateLOD(CurrentOwnerLODIndex))
		{
			return ClothingSimulation->GetBounds(OwnerComponent);
		}
		return FBoxSphereBounds(ForceInit);
	}

private:
	/** ClothingSimulations are responsible for maintaining and simulating clothing actors. */
	IClothingSimulationInterface* ClothingSimulation = nullptr;

	/** ClothingSimulationContexts are a datastore for simulation data sent to the clothing thread. */
	IClothingSimulationContext* ClothingSimulationContext = nullptr;

	/**
	 * Objects responsible for interacting with the clothing simulation.
	 * Blueprints and code can call/set data on this from the game thread and the next time
	 * it is safe to do so the interactor will sync to the simulation context.
	 */
	UPROPERTY(Transient)
	TObjectPtr<UClothingSimulationInteractor> ClothingSimulationInteractor;

	/** Simulation factory. */
	UPROPERTY(Transient)
	TObjectPtr<UClothingSimulationFactory> ClothingSimulationFactory;

	int32 CurrentOwnerLODIndex = INDEX_NONE; // This is populated by FillContextAndPrepareTick
};

template<>
struct TStructOpsTypeTraits<FClothingSimulationInstance> : public TStructOpsTypeTraitsBase2<FClothingSimulationInstance>
{
	enum
	{
		WithCopy = false
	};
};

#undef UE_API
