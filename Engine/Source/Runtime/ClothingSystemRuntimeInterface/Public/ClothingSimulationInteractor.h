// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Math/MathFwd.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "ClothingSimulationInterface.h"

#include "ClothingSimulationInteractor.generated.h"

#define UE_API CLOTHINGSYSTEMRUNTIMEINTERFACE_API

class FName;
class IClothingSimulationContext;
class UClothingAssetBase;
struct FFrame;
template <typename T> struct TObjectPtr;

/**
 * Abstract class to control clothing specific interaction.
 * Must be cast to the end used clothing simulation object before use.
 */
UCLASS(Abstract, BlueprintType, MinimalAPI)
class UClothingInteractor : public UObject
{
	GENERATED_BODY()

protected:
	friend class UClothingSimulationInteractor;

	virtual void Sync(IClothingSimulationInterface* Simulation)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Simulation && Simulation->DynamicCastToIClothingSimulation())
		{
			Sync(Simulation->DynamicCastToIClothingSimulation());
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	virtual void Sync(IClothingSimulation* Simulation) {}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

protected:
	int32 ClothingId = -1;  // Clothing Asset slot as passed to the CreateActor function
};

/** 
 * If a clothing simulation is able to be interacted with at runtime then a derived
 * interactor should be created, and at least the basic API implemented for that
 * simulation.
 * Only write to the simulation and context during the call to Sync, as that is
 * guaranteed to be a safe place to access this data.
 */
UCLASS(Abstract, BlueprintType, MinimalAPI)
class UClothingSimulationInteractor : public UObject
{
	GENERATED_BODY()

public:
	/** Create a cloth specific interactor. Override either this method or the simpler CreateClothingInteractor method. */
	UE_API virtual void CreateClothingInteractor(const UClothingAssetBase* ClothingAsset, int32 ClothingId);

	/** Destroy all interactors. */
	UE_API void DestroyClothingInteractors();

	// Basic interface that clothing simulations are required to support

	/**
	 * Sync the interactor with the clothing simulation to use any changes made on the next simulate.
	 * This will be called at a time that is guaranteed to never overlap with the running simulation.
	 * The simulation will have to be written in a way to pick up these changes on the next update.
	 * Any inherited class must call this function to also Sync the ClothingInteractors.
	 */
	UE_API virtual void Sync(IClothingSimulationInterface* Simulation, const IClothingSimulationContext* Context);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
	UE_API virtual void Sync(IClothingSimulation* Simulation, IClothingSimulationContext* Context);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Called to update collision status without restarting the simulation. */
	UFUNCTION(BlueprintCallable, Category=ClothingSimulation)
	virtual void PhysicsAssetUpdated()
	PURE_VIRTUAL(UClothingSimulationInteractor::PhysicsAssetUpdated, );

	/** Called to update the cloth config without restarting the simulation. */
	UFUNCTION(BlueprintCallable, Category=ClothingSimulation)
	virtual void ClothConfigUpdated()
	PURE_VIRTUAL(UClothingSimulationInteractor::ClothConfigUpdated, );

	/** Set the stiffness of the spring force for the animation drive. */
	UFUNCTION(BlueprintCallable, Category = ClothingSimulation)
	virtual void SetAnimDriveSpringStiffness(float InStiffness)
	PURE_VIRTUAL(UClothingSimulationInteractor::SetAnimDriveSpringStiffness, );

	/** Set a new gravity override and enable the override. */
	UFUNCTION(BlueprintCallable, Category = ClothingSimulation)
	virtual void EnableGravityOverride(const FVector& InVector)
	PURE_VIRTUAL(UClothingSimulationInteractor::EnableGravityOverride, );

	/** Disable any currently set gravity override. */
	UFUNCTION(BlueprintCallable, Category = ClothingSimulation)
	virtual void DisableGravityOverride()
	PURE_VIRTUAL(UClothingSimulationInteractor::DisableGravityOverride, );

	/** Set the number of time dependent solver iterations. */
	UFUNCTION(BlueprintCallable, Category = ClothingSimulation)
	virtual void SetNumIterations(int32 NumIterations = 1)
	PURE_VIRTUAL(UClothingSimulationInteractor::SetNumIterations, );

	/** Set the maximum number of solver iterations. */
	UFUNCTION(BlueprintCallable, Category = ClothingSimulation)
	virtual void SetMaxNumIterations(int32 MaxNumIterations = 10)
	PURE_VIRTUAL(UClothingSimulationInteractor::SetMaxNumIterations, );

	/** Set the number of substeps or subdivisions. */
	UFUNCTION(BlueprintCallable, Category = ClothingSimulation, Meta = (Keywords = "Subdivisions"))
	virtual void SetNumSubsteps(int32 NumSubsteps = 1)
	PURE_VIRTUAL(UClothingSimulationInteractor::SetNumSubsteps, );

	// Base clothing simulations interface

	/** Return the number of cloths run by the simulation. */
	UFUNCTION(BlueprintCallable, Category = ClothingSimulation)
	int32 GetNumCloths() const { return LastNumCloths; }

	/** Return the number of kinematic (animated) particles. */
	UFUNCTION(BlueprintCallable, Category = ClothingSimulation)
	int32 GetNumKinematicParticles() const { return LastNumKinematicParticles; }

	/** Return the number of dynamic (simulated) particles. */
	UFUNCTION(BlueprintCallable, Category = ClothingSimulation)
	int32 GetNumDynamicParticles() const { return LastNumDynamicParticles; }

	/**
	 * Return the solver number of iterations.
	 * This could be different from the number set if the simulation hasn't updated yet.
	 */
	UFUNCTION(BlueprintCallable, Category = ClothingSimulation)
	int32 GetNumIterations() const { return LastNumIterations; }

	/**
	 * Return the solver number of subdivisions./
	 * This could be different from the number set if the simulation hasn't updated yet.
	 */
	UFUNCTION(BlueprintCallable, Category = ClothingSimulation)
	int32 GetNumSubsteps() const { return LastNumSubsteps; }

	/** Return the instant average simulation time in ms. */
	UFUNCTION(BlueprintCallable, Category = ClothingSimulation)
	float GetSimulationTime() const { return LastSimulationTime; }

	/**Return a cloth interactor for this simulation. */
	UFUNCTION(BlueprintCallable, Category = ClothingSimulation)
	UE_API UClothingInteractor* GetClothingInteractor(const FName ClothingAssetName) const;

	UE_DEPRECATED(5.7, "Use FName version instead.")
	UClothingInteractor* GetClothingInteractor(const FString& ClothingAssetName) const
	{
		return GetClothingInteractor(FName(ClothingAssetName));
	}

	/** Cloth interactors currently created. */
	UPROPERTY()
	TMap<FName, TObjectPtr<UClothingInteractor>> ClothingInteractors;

protected:
	/** Create a clothing interactor. This method is called by the public CreateClothingInteractor() default implementation. */
	virtual UClothingInteractor* CreateClothingInteractor()
	{
		return nullptr;
	}

private:
	// Simulation infos updated during the sync
	int32 LastNumCloths = 0;
	int32 LastNumKinematicParticles = 0;
	int32 LastNumDynamicParticles = 0;
	int32 LastNumIterations = 0;
	int32 LastNumSubsteps = 0;
	float LastSimulationTime = 0.f;
};

#undef UE_API
