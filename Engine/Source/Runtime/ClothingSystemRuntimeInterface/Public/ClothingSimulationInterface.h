// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClothingSystemRuntimeTypes.h"
#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Math/MathFwd.h"

#define UE_API CLOTHINGSYSTEMRUNTIMEINTERFACE_API

class UClothingAssetBase;
class USkeletalMeshComponent;
class USkinnedMeshComponent;
class UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.") IClothingSimulation;
struct FClothCollisionData;
struct FClothSimulData;

/** Empty interface, derived simulation modules define the contents of the context. */
class IClothingSimulationContext
{
public:
	UE_API IClothingSimulationContext();
	UE_API virtual ~IClothingSimulationContext();
};

/**
 * Base class for clothing simulators.
 * 
 * The majority of the API for this class is protected.
 * For the most part the simulation is not designed to be used outside of the FClothingSimulationInstance implementation which is
 * used in the Skeletal Mesh Component, as its parallel simulation is tied to the Skeletal Mesh Component tick and dependents.
 * Any method that is available in the public section below should consider that it may be called while the simulation is running.
 */
class IClothingSimulationInterface
{
public:
	UE_API IClothingSimulationInterface();
	UE_API virtual ~IClothingSimulationInterface();

protected:

	friend struct FClothingSimulationInstance;

	/**
	 * Create an actor for this simulation from the data in InAsset
	 * Simulation data for this actor should be written back to SimDataIndex in AppendSimulationData
	 * @param InOwnerComponent - The component requesting this actor
	 * @param InAsset - The asset to create an actor from
	 * @param SimDataIndex - the sim data index to use when doing the writeback in AppendSimulationData
	 */
	virtual void CreateActor(USkeletalMeshComponent* InOwnerComponent, const UClothingAssetBase* InAsset, int32 SimDataIndex) = 0;

	/** Called once all CreateActor have been called. */
	virtual void EndCreateActor() = 0;

	/** Create a new context, will not be filled, call FillContextAndPrepareTick before simulating with this context */
	virtual IClothingSimulationContext* CreateContext() = 0;

	/**
	 * Fills an existing context for a single simulation step and do any other work that needs to be called by the engine on the game thread prior to simulation.
	 * @param InComponent - The component to fill the context for
	 * @param InOutContext - The context to fill
	 * @param bIsInitialization - Whether this fill is occurring as part of the actor creation stage
	 * @param bForceTeleportResetOnly - Whether ForceClothNextUpdateTeleportAndReset instead of Simulate and AppendSimulationData will be called.
	 */
	virtual void FillContextAndPrepareTick(const USkeletalMeshComponent* InComponent, float InDeltaTime, IClothingSimulationContext* InOutContext, bool bIsInitialization, bool bForceTeleportResetOnly) = 0;

	/** Initialize the simulation, will be called before any Simulate calls */
	virtual void Initialize() = 0;

	/**
	 * Shutdown the simulation, this should clear ALL resources as we no longer expect to
	 * call any other function on this simulation without first calling Initialize again.
	 */
	virtual void Shutdown() = 0;

	/**
	 * Called by the engine to determine if this simulation can run this tick at this LOD.
	 * ForceClothNextUpdateTeleportAndReset rather than Simulate and AppendSimulationData will be called this tick.
	 */
	virtual bool ShouldSimulateLOD(int32 OwnerLODIndex) const = 0;

	/**
	 * Run a single tick of the simulation.
	 * The pointer InContext is guaranteed (when called by the engine) to be the context allocated in
	 * CreateContext and can be assumed to be safely castable to any derived type allocated there.
	 * New callers should take care to make sure only the correct context is ever passed through.
	 * @note Can be called asynchronously outside of the game thread.
	 * @param InContext - The context to use during simulation, will have been filled in FillContextAndPrepareTick
	 */
	virtual void Simulate_AnyThread(const IClothingSimulationContext* InContext) = 0;

	/**
	 * Called instead of Simulate when not ticking.
	 * @note Can be called asynchronously outside of the game thread.
	 */
	virtual void ForceClothNextUpdateTeleportAndReset_AnyThread() = 0;

	/** Hard reset the simulation without necessarily recreating cloth actors. */
	virtual void HardResetSimulation(const IClothingSimulationContext* InContext) = 0;

	/** Simulation should remove all of it's actors when next possible and free them */
	virtual void DestroyActors() = 0;

	/**
	 * Destroy a context object, engine will always pass a context created using CreateContext
	 * @param InContext - The context to destroy
	 */
	virtual void DestroyContext(IClothingSimulationContext* InContext) = 0;

	/**
	 * Fill FClothSimulData map for the clothing simulation. Should fill a map pair per-actor.
	 * Do not remove InOutData elements which you are not responsible for: they may have been written by a different instance.
	 * @param InOutData - The simulation data to write to
	 * @param OwnerComponent - the component that owns the simulation
	 * @param OverrideComponent - An override component if bound to a leader pose component
	 */
	virtual void AppendSimulationData(TMap<int32, FClothSimulData>& InOutData, const USkeletalMeshComponent* OwnerComponent, const USkinnedMeshComponent* OverrideComponent) const = 0;

	/** Get the bounds of the simulation mesh in local simulation space */
	virtual FBoxSphereBounds GetBounds(const USkeletalMeshComponent* InOwnerComponent) const = 0;

	/**
	 * Called by the engine when an external object wants to inject collision data into this simulation above
	 * and beyond what is specified in the asset for the internal actors
	 * Examples: Scene collision, collision for parents we are attached to
	 * @param InData - Collisions to add to this simulation
	 */
	virtual void AddExternalCollisions(const FClothCollisionData& InData) = 0;

	/**
	 * Called by the engine when external collisions are no longer necessary or when they need to be updated
	 * with some of the previous collisions removed. It is recommended in derived simulations to avoid freeing
	 * any allocations regarding external collisions as it is likely more will be added soon after this call
	 */
	virtual void ClearExternalCollisions() = 0;

	UE_DEPRECATED(5.8, "GetCollisions has been deprecated. Use CollisionSources to add external collisions instead.")
	virtual void GetCollisions(FClothCollisionData& OutCollisions, bool bIncludeExternal = true) const {};

public:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "This method is used to ease transition between the new and old interface and will be removed.")
	inline const IClothingSimulation* DynamicCastToIClothingSimulation() const;

	UE_DEPRECATED(5.7, "This method is used to ease transition between the new and old interface and will be removed.")
	inline IClothingSimulation* DynamicCastToIClothingSimulation();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Return the number of simulated cloths. Implementation must be thread safe. */
	virtual int32 GetNumCloths() const { return 0; }
	/** Return the number of kinematic (fixed) particles. Implementation must be thread safe. */
	virtual int32 GetNumKinematicParticles() const { return 0; }
	/** Return the number of dynamic (simulated) particles. Implementation must be thread safe. */
	virtual int32 GetNumDynamicParticles() const { return 0; }
	/**
	 * Return the number of iterations used by the solver.
	 * This is the maximum used as an indicative value only, as this could vary between cloths.
	 * Implementation must be thread safe.
	 */
	virtual int32 GetNumIterations() const { return 0; }
	/**
	 * Return the number of substeps used by the solver.
	 * This is the maximum used as an indicative value only, as this could vary between cloths.
	 * Implementation must be thread safe.
	 */
	virtual int32 GetNumSubsteps() const { return 0; }
	/** Return the simulation time in ms. Implementation must be thread safe. */
	virtual float GetSimulationTime() const { return 0.f; }
	/** Return whether the simulation is teleported. Implementation must be thread safe. */
	virtual bool IsTeleported() const { return false; }

private:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	friend class IClothingSimulation;  // For bIsLegacyInterface
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	bool bIsLegacyInterface = false;  // UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.")
};

class UE_DEPRECATED(5.7, "Use IClothingSimulationInterface instead.") IClothingSimulation : public IClothingSimulationInterface
{
public:
	UE_API IClothingSimulation();
	UE_API virtual ~IClothingSimulation();

	// The majority of the API for this class is protected. The required objects (skel meshes and the parallel task)
	// are friends so they can use the functionality. For the most part the simulation is not designed to be used
	// outside of the skeletal mesh component as its parallel simulation is tied to the skel mesh tick and
	// dependents.
	// Any method that is available in the public section below should consider that it may be called while
	// the simulation is running.
	// Currently only a const pointer is exposed externally to the skeletal mesh component, so barring that
	// being cast away external callers can't call non-const methods. The debug skel mesh component exposes a
	// mutable version for editor use
protected:

	friend class USkeletalMeshComponent;
	friend class FParallelClothTask;
	friend struct FClothingSimulationInstance;
	friend struct FEnvironmentalCollisions;

	/**
	 * Create an actor for this simulation from the data in InAsset
	 * Simulation data for this actor should be written back to SimDataIndex in AppendSimulationData
	 * @param InOwnerComponent - The component requesting this actor
	 * @param InAsset - The asset to create an actor from
	 * @param SimDataIndex - the sim data index to use when doing the writeback in AppendSimulationData
	 */
	virtual void CreateActor(USkeletalMeshComponent* InOwnerComponent, const UClothingAssetBase* InAsset, int32 SimDataIndex) override
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CreateActor(InOwnerComponent, const_cast<UClothingAssetBase*>(InAsset), SimDataIndex);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.7, "Use const arguments version instead,")
	virtual void CreateActor(USkeletalMeshComponent* InOwnerComponent, UClothingAssetBase* InAsset, int32 SimDataIndex) {}

	/**
	 * Fills an existing context for a single simulation step and do any other work that needs to be called by the engine on the game thread prior to simulation.
	 * @param InComponent - The component to fill the context for
	 * @param InOutContext - The context to fill
	 * @param bIsInitialization - Whether this fill is occurring as part of the actor creation stage
	 * @param bForceTeleportResetOnly - Whether ForceClothNextUpdateTeleportAndReset instead of Simulate and AppendSimulationData will be called.
	 */
	virtual void FillContextAndPrepareTick(const USkeletalMeshComponent* InComponent, float InDeltaTime, IClothingSimulationContext* InOutContext, bool bIsInitialization, bool bForceTeleportResetOnly) override
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return FillContext(const_cast<USkeletalMeshComponent*>(InComponent), InDeltaTime, InOutContext, bIsInitialization);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.7, "Use FillContextAndPrepareTick instead.")
	virtual void FillContext(USkeletalMeshComponent* InComponent, float InDeltaTime, IClothingSimulationContext* InOutContext, bool bIsInitialization)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return FillContext(InComponent, InDeltaTime, InOutContext);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	UE_DEPRECATED(5.7, "Use FillContextAndPrepareTick instead.")
	virtual void FillContext(USkeletalMeshComponent* InComponent, float InDeltaTime, IClothingSimulationContext* InOutContext) {}

	/**
	 * Called by the engine to detect whether or not the simulation should run, essentially
	 * are there any actors that need to simulate in this simulation
	 */
	UE_DEPRECATED(5.7, "This method has not been used consistently and has been replaced with ShouldSimulateLOD")
	virtual bool ShouldSimulate() const
	{
		return true;
	}

	/** Called by the engine to determine if this simulation can run this tick at this LOD.
	 *  ForceClothNextUpdateTeleportAndReset rather than Simulate and AppendSimulationData will be called this tick.
	 */
	virtual bool ShouldSimulateLOD(int32 OwnerLODIndex) const override
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ShouldSimulate();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Run a single tick of the simulation.
	 * The pointer InContext is guaranteed (when called by the engine) to be the context allocated in
	 * CreateContext and can be assumed to be safely castable to any derived type allocated there.
	 * New callers should make care to make sure only the correct context is ever passed through
	 * @param InContext - The context to use during simulation, will have been filled in FillContextAndPrepareTick
	 */
	virtual void Simulate_AnyThread(const IClothingSimulationContext* InContext) override
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Simulate(const_cast<IClothingSimulationContext*>(InContext));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	UE_DEPRECATED(5.7, "Use Simulate_AnyThread instead,")
	virtual void Simulate(IClothingSimulationContext* InContext) {}

	/** Called instead of Simulate when not ticking.*/
	virtual void ForceClothNextUpdateTeleportAndReset_AnyThread() override {}

	/** Hard reset the simulation without necessarily recreating cloth actors. */
	virtual void HardResetSimulation(const IClothingSimulationContext* InContext) override {}

	/** 
	 * Fill FClothSimulData map for the clothing simulation. Should fill a map pair per-actor. 
	 * Do not remove InOutData elements which you are not responsible for: they may have been written by a different instance.
	 * @param InOutData - The simulation data to write to
	 * @param OwnerComponent - the component that owns the simulation
	 * @param OverrideComponent - An override component if bound to a leader pose component
	 */
	virtual void AppendSimulationData(TMap<int32, FClothSimulData>& InOutData, const USkeletalMeshComponent* OwnerComponent, const USkinnedMeshComponent* OverrideComponent) const override
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		GetSimulationData(InOutData, const_cast<USkeletalMeshComponent*>(OwnerComponent), const_cast<USkinnedMeshComponent*>(OverrideComponent));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.7, "Use AppendSimulationData instead.")
	virtual void GetSimulationData(TMap<int32, FClothSimulData>& OutData, USkeletalMeshComponent* InOwnerComponent, USkinnedMeshComponent* InOverrideComponent) const {}

public:
	UE_DEPRECATED(5.7, "GatherStats is unused and will be removed from future interfaces. Do not override.")
	virtual void GatherStats() const {}

	UE_DEPRECATED(5.7, "Use a clothing simulation interactor to set this simulation property instead.")
	virtual void SetNumIterations(int32 NumIterations) {}
	UE_DEPRECATED(5.7, "Use a clothing simulation interactor to set this simulation property instead.")
	virtual void SetMaxNumIterations(int32 MaxNumIterations) {}
	UE_DEPRECATED(5.7, "Use a clothing simulation interactor to set this simulation property instead.")
	virtual void SetNumSubsteps(int32 NumSubsteps) {}

	UE_DEPRECATED(5.7, "Use FillContextAndPrepareTick instead.")
	virtual void UpdateWorldForces(const USkeletalMeshComponent* OwnerComponent) {}
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
inline const class IClothingSimulation* IClothingSimulationInterface::DynamicCastToIClothingSimulation() const
{
	return bIsLegacyInterface ? static_cast<const class IClothingSimulation*>(this) : nullptr;
}

inline IClothingSimulation* IClothingSimulationInterface::DynamicCastToIClothingSimulation()
{
	return bIsLegacyInterface ? static_cast<class IClothingSimulation*>(this) : nullptr;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API
