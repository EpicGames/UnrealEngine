// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/UnrealMathUtility.h"

enum class EClothingTeleportMode : uint8;

namespace UE::ClothingSimulation::TeleportHelpers
{
	inline float ComputeTeleportCosineRotationThreshold(float ThresholdInDegrees)
	{
		// Threshold <= 0 disables check.
		// Cos(0) = 1, so set to 1 for all negative values.
		// CalculateClothingTeleport will use ClothTeleportCosineThreshold >= 1 to disable.
		return ThresholdInDegrees > 0.f ? FMath::Cos(FMath::DegreesToRadians(ThresholdInDegrees)) : 1.f;
	}

	inline float ComputeTeleportDistanceThresholdSquared(float ThresholdDistance)
	{
		// Threshold <= 0 disables check.
		return ThresholdDistance > 0.f ? FMath::Square(ThresholdDistance) : 0.f;
	}

	CLOTHINGSYSTEMRUNTIMEINTERFACE_API EClothingTeleportMode CalculateClothingTeleport(EClothingTeleportMode CurrentTeleportMode, const FMatrix& CurRootBoneMat, const FMatrix& PrevRootBoneMat, bool bResetAfterTeleport, float ClothTeleportDistThresholdSquared, float ClothTeleportCosineThresholdInRad);
};
