// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class FClothingSystemRuntimeInterfaceModule : public IModuleInterface
{

public:

	FClothingSystemRuntimeInterfaceModule();

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
};
