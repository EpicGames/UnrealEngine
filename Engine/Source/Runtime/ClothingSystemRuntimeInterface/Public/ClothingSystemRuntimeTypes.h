// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "ClothingSystemRuntimeTypes.generated.h"


// Data produced by a clothing simulation
struct FClothSimulData
{
	void Reset()
	{
		Positions.Reset();
		Normals.Reset();
		LODIndex = INDEX_NONE;
	}

	bool IsReset() const
	{
		return Positions.IsEmpty() && Normals.IsEmpty() && LODIndex == INDEX_NONE;
	}

	// Positions of the simulation mesh particles (aligned for SIMD loads)
	TArray<FVector3f, TAlignedHeapAllocator<16>> Positions;

	// Normals at the simulation mesh particles (aligned for SIMD loads)
	TArray<FVector3f, TAlignedHeapAllocator<16>> Normals;

	// Transform applied per position/normal element when loaded
	FTransform Transform;

	// Transform relative to the component to update clothing root transform when not ticking clothing but rendering a component
	FTransform ComponentRelativeTransform;

	// Current LOD index the data is valid for
	int32 LODIndex;
};

UENUM()
enum class EClothingTeleportMode : uint8
{
	// No teleport, simulate as normal
	None = 0,
	// Teleport the simulation, causing no intertial effects but keep the sim mesh shape
	Teleport,
	// Teleport the simulation, causing no intertial effects and reset the sim mesh shape
	TeleportAndReset,
	// Hard reset the simulation by refreshing the cloth config
	HardReset
};

struct IClothSimulationDataProvider
{
	virtual void GetUpdateClothSimulationData_AnyThread(TMap<int32, FClothSimulData>& OutClothSimulData, FMatrix& OutLocalToWorld, float& OutClothBlendWeight) const = 0;
};